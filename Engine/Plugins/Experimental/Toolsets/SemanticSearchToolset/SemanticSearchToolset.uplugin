{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Semantic Search Toolset",
	"Description": "Exposes the SemanticSearch plugin's hybrid vector+BM25 asset search as an AI Toolset Registry tool.",
	"Category": "Other",
	"EditorOnly": true,
	"IsExperimentalVersion": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SemanticSearchToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "SemanticSearch",
			"Enabled": true
		}
	]
}
