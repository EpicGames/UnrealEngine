// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SemanticSearchToolset.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"

#include "SemanticSearchAsyncResult.generated.h"

// Async tool call result that completes with an array of FSemanticSearchResult.
UCLASS()
class USemanticSearchAsyncResult : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	UFUNCTION()
	bool SetValue(const TArray<FSemanticSearchResult>& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(TArray<FSemanticSearchResult>(InValue), Value);
	}

	bool SetValue(TArray<FSemanticSearchResult>&& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(MoveTemp(InValue), Value);
	}

	UPROPERTY()
	TArray<FSemanticSearchResult> Value;
};
