// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchFilter.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::SemanticSearchToolset
{
	int32 ExpandClassFilter(
		const TArray<UClass*>& ClassFilter,
		const TSet<FName>& SupportedClassNames,
		TSet<FName>& OutAllowedShortNames)
	{
		if (ClassFilter.IsEmpty())
		{
			return 0;
		}

		IAssetRegistry& AssetRegistry =
			FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		int32 Unresolved = 0;
		for (const UClass* Class : ClassFilter)
		{
			if (!Class)
			{
				++Unresolved;
				continue;
			}
			TSet<FTopLevelAssetPath> Derived;
			AssetRegistry.GetDerivedClassNames(
				{ Class->GetClassPathName() }, TSet<FTopLevelAssetPath>(), Derived);

			bool bAnyIntersected = false;
			for (const FTopLevelAssetPath& Path : Derived)
			{
				const FName ShortName = Path.GetAssetName();
				if (SupportedClassNames.Contains(ShortName))
				{
					OutAllowedShortNames.Add(ShortName);
					bAnyIntersected = true;
				}
			}
			if (!bAnyIntersected)
			{
				++Unresolved;
			}
		}

		return Unresolved;
	}

	bool MatchesPathRegex(const FString& SoftObjectPath, const TArray<FRegexPattern>& CompiledPatterns)
	{
		if (CompiledPatterns.IsEmpty())
		{
			return true;
		}

		for (const FRegexPattern& Pattern : CompiledPatterns)
		{
			FRegexMatcher Matcher(Pattern, SoftObjectPath);
			if (Matcher.FindNext())
			{
				return true;
			}
		}
		return false;
	}

	int32 CompileRegexPatterns(const TArray<FString>& RegexStrings, TArray<FRegexPattern>& OutPatterns)
	{
		int32 FailedCount = 0;
		OutPatterns.Reserve(OutPatterns.Num() + RegexStrings.Num());
		for (const FString& RegexString : RegexStrings)
		{
			if (RegexString.IsEmpty())
			{
				++FailedCount;
				continue;
			}
			// FRegexPattern has no explicit failure path on invalid syntax in UE's ICU-backed
			// implementation — construction succeeds and matching returns false. Empty strings
			// are the main degenerate case we pre-screen above.
			OutPatterns.Emplace(RegexString);
		}
		return FailedCount;
	}
}
