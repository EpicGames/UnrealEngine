// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Regex.h"

struct FAssetData;

namespace UE::SemanticSearchToolset
{
	/** Resolve user-provided short class names to the set of concrete class short names whose
	 *  assets should be considered, including subclasses.

	 *  @return number of ClassFilter entries that could not be resolved (unknown name, empty string,
	 *    or resolved to a class that isn't in SupportedClassNames). Caller may log this.
	 */
	int32 ExpandClassFilter(
		const TArray<UClass*>& ClassFilter,
		const TSet<FName>& SupportedClassNames,
		TSet<FName>& OutAllowedShortNames);

	/** Returns true if SoftObjectPath matches any of the precompiled regex patterns.
	 *  An empty list accepts any path.
	 */
	bool MatchesPathRegex(
		const FString& SoftObjectPath,
		const TArray<FRegexPattern>& CompiledPatterns);

	/** Compile the given regex strings. Invalid patterns are skipped (not added to the output).
	 *  Returns the number of patterns that failed to compile.
	 */
	int32 CompileRegexPatterns(
		const TArray<FString>& RegexStrings,
		TArray<FRegexPattern>& OutPatterns);
}
