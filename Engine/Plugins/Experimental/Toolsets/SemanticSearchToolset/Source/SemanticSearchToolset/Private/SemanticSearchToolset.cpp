// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchToolset.h"

#include "Async/Async.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetProcessorManager.h"
#include "HybridSearchIndex.h"
#include "ISemanticSearchModule.h"
#include "Interfaces/IEmbeddingProvider.h"
#include "Logging/LogMacros.h"
#include "SemanticSearchFilter.h"
#include "SemanticSearchAsyncResult.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"

DEFINE_LOG_CATEGORY_STATIC(LogSemanticSearchToolset, Log, All);

namespace UE::SemanticSearchToolset::Private
{
	struct FCandidateSet
	{
		TArray<int64> IDFilter;
		TMap<int64, FAssetData> IDToAsset;
	};

	/** Iterate the AssetRegistry once and build the candidate ID filter that matches
	 *  the indexed-folder + class + path constraints.
	 *
	 *  AllowedClassShortNames: if non-empty, only assets whose concrete short class name is in
	 *  this set are considered. Empty means "no class filter" (all indexed classes considered). */
	static FCandidateSet BuildCandidateSet(
		const TSet<FName>& AllowedClassShortNames,
		const TArray<FRegexPattern>& CompiledRegexes)
	{
		FCandidateSet Out;

		IAssetRegistry& AssetRegistry =
			FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		const bool bHasClassFilter = !AllowedClassShortNames.IsEmpty();

		AssetRegistry.EnumerateAllAssets([&](const FAssetData& AssetData)
		{
			if (AssetData.IsRedirector())
			{
				return true;
			}
			if (!UE::SemanticSearch::ISemanticSearchModule::IsInIndexedFolder(AssetData))
			{
				return true;
			}

			if (bHasClassFilter && !AllowedClassShortNames.Contains(AssetData.AssetClassPath.GetAssetName()))
			{
				return true;
			}

			const FString SoftPath = AssetData.GetSoftObjectPath().ToString();
			if (!UE::SemanticSearchToolset::MatchesPathRegex(SoftPath, CompiledRegexes))
			{
				return true;
			}

			const int64 ID = UE::SemanticSearch::GetAssetIndexID(AssetData);
			Out.IDFilter.Add(ID);
			Out.IDToAsset.Add(ID, AssetData);
			return true;
		});

		return Out;
	}

	/** Shared per-request state, held alive by all async callbacks via TSharedRef. */
	struct FSearchRequest
	{
		TStrongObjectPtr<USemanticSearchAsyncResult> AsyncResult;
		FString Query;
		int32 K = 0;
	};

	/** Fan-in state for caption fetches. Owned by a TSharedRef captured in each GetCaptionData callback. */
	struct FCaptionFanIn
	{
		TArray<FSemanticSearchResult> Results;
		int32 Remaining = 0;
		TSharedRef<FSearchRequest> Request;

		explicit FCaptionFanIn(TSharedRef<FSearchRequest> InRequest)
			: Request(MoveTemp(InRequest))
		{
		}
	};

	static void BuildResultsAndFetchCaptions(
		TSharedRef<FSearchRequest> Request,
		TArray<UE::SemanticSearch::FHybridSearchResult>&& Hits,
		TMap<int64, FAssetData>&& IDToAsset)
	{
		TArray<FSemanticSearchResult> Results;
		TArray<FAssetData> AssetsToCaption;
		Results.Reserve(Hits.Num());
		AssetsToCaption.Reserve(Hits.Num());

		for (const UE::SemanticSearch::FHybridSearchResult& Hit : Hits)
		{
			const FAssetData* AssetDataPtr = IDToAsset.Find(Hit.ID);
			if (AssetDataPtr == nullptr)
			{
				// Asset was deleted between enumeration and result delivery.
				continue;
			}

			FSemanticSearchResult& Out = Results.AddDefaulted_GetRef();
			Out.Path = AssetDataPtr->GetSoftObjectPath();
			Out.Class = AssetDataPtr->GetClass(EResolveClass::Yes);
			AssetsToCaption.Add(*AssetDataPtr);
		}

		if (Results.IsEmpty())
		{
			Request->AsyncResult->SetValue(MoveTemp(Results));
			return;
		}

		TSharedRef<FCaptionFanIn> FanIn = MakeShared<FCaptionFanIn>(Request);
		const int32 NumResults = Results.Num();
		FanIn->Results = MoveTemp(Results);
		// Sentinel: start at N+1 and release the extra count after dispatching all callbacks.
		// Prevents a cache-hit synchronous callback from finalizing (and moving FanIn->Results)
		// mid-loop before later iterations have been dispatched.
		FanIn->Remaining = NumResults + 1;

		for (int32 Index = 0; Index < NumResults; ++Index)
		{
			const FAssetData& Asset = AssetsToCaption[Index];
			const int32 CapturedIndex = Index;
			const bool bDispatched = UE::SemanticSearch::FAssetProcessorManager::Get().GetCaptionData(
				Asset,
				[FanIn, CapturedIndex](UE::SemanticSearch::FAssetCaptionResult&& CaptionResult, FString&& Error, UE::SemanticSearch::EAssetIndexFailureReason Reason) mutable
				{
					auto Deliver = [FanIn, CapturedIndex, CaptionResult = MoveTemp(CaptionResult), Error = MoveTemp(Error)]() mutable
					{
						if (Error.IsEmpty())
						{
							FanIn->Results[CapturedIndex].Caption = MoveTemp(CaptionResult.Caption);
						}
						if (--FanIn->Remaining == 0)
						{
							FanIn->Request->AsyncResult->SetValue(MoveTemp(FanIn->Results));
						}
					};

					if (IsInGameThread())
					{
						Deliver();
					}
					else
					{
						AsyncTask(ENamedThreads::GameThread, [Deliver = MoveTemp(Deliver)]() mutable { Deliver(); });
					}
				});

			if (!bDispatched)
			{
				// No processor for this asset class — caption stays empty. Count it down;
				// the sentinel guarantees this cannot reach zero mid-loop.
				--FanIn->Remaining;
			}
		}

		// Release the sentinel. If every callback already fired synchronously, this drops to 0.
		if (--FanIn->Remaining == 0)
		{
			FanIn->Request->AsyncResult->SetValue(MoveTemp(FanIn->Results));
		}
	}

	/** K validation + class-filter expansion + candidate-set construction, shared between
	 *  Search and FindSimilar. Returns true if the caller should proceed; returns false after
	 *  having resolved the async result (error or empty). */
	static bool PrepareCandidates(
		int32 InK,
		const TArray<UClass*>& ClassFilter,
		const TArray<FString>& PathRegexes,
		USemanticSearchAsyncResult& Result,
		const TCHAR* ToolName,
		FCandidateSet& OutCandidates)
	{
		if (InK <= 0)
		{
			Result.SetError(TEXT("K must be >= 1."));
			return false;
		}

		TArray<FRegexPattern> CompiledRegexes;
		const int32 RegexFailures = CompileRegexPatterns(PathRegexes, CompiledRegexes);
		if (RegexFailures > 0)
		{
			UE_LOG(LogSemanticSearchToolset, Warning,
				TEXT("SemanticSearchToolset::%s: %d path regex(es) were empty and skipped."),
				ToolName, RegexFailures);
		}

		// Expand the user's class-name filter to the full set of indexed concrete subclass short
		// names. E.g. "Texture" expands to {"Texture2D", "TextureCube", ...} intersected with
		// what the SemanticSearch plugin actually processes.
		TSet<FName> AllowedClassShortNames;
		if (!ClassFilter.IsEmpty())
		{
			const TSet<FName> SupportedClassNames =
				UE::SemanticSearch::FAssetProcessorManager::Get().GetSupportedClassNames();

			const int32 Unresolved = ExpandClassFilter(ClassFilter, SupportedClassNames, AllowedClassShortNames);
			if (Unresolved > 0)
			{
				UE_LOG(LogSemanticSearchToolset, Warning,
					TEXT("SemanticSearchToolset::%s: %d ClassFilter entry(ies) were unknown or not indexed."),
					ToolName, Unresolved);
			}
			if (AllowedClassShortNames.IsEmpty())
			{
				// Every provided name was unresolvable or outside the indexed set — nothing can match.
				Result.SetValue(TArray<FSemanticSearchResult>());
				return false;
			}
		}

		OutCandidates = BuildCandidateSet(AllowedClassShortNames, CompiledRegexes);

		// If the caller specified any filter and nothing matched, short-circuit to an empty result.
		const bool bHasFilter = !AllowedClassShortNames.IsEmpty() || !CompiledRegexes.IsEmpty();
		if (bHasFilter && OutCandidates.IDFilter.IsEmpty())
		{
			Result.SetValue(TArray<FSemanticSearchResult>());
			return false;
		}

		return true;
	}

	static void RunSearchWithEmbedding(
		TSharedRef<FSearchRequest> Request,
		TArray<float>&& QueryEmbedding,
		TArray<int64>&& IDFilter,
		TMap<int64, FAssetData>&& IDToAsset)
	{
		UE::SemanticSearch::ISemanticSearchModule::Get().SearchAsync(
			Request->Query,
			QueryEmbedding,
			Request->K,
			IDFilter,
			TNumericLimits<float>::Max(),
			[Request, IDToAsset = MoveTemp(IDToAsset)]
			(TArray<UE::SemanticSearch::FHybridSearchResult>&& Results) mutable
			{
				// SearchAsync guarantees game-thread delivery.
				BuildResultsAndFetchCaptions(Request, MoveTemp(Results), MoveTemp(IDToAsset));
			});
	}
}

USemanticSearchAsyncResult* USemanticSearchToolset::Search(
	const FString& Query,
	const TArray<UClass*>& ClassFilter,
	const TArray<FString>& PathRegexes,
	int32 K)
{
	using namespace UE::SemanticSearchToolset;
	using namespace UE::SemanticSearchToolset::Private;

	USemanticSearchAsyncResult* Result = NewObject<USemanticSearchAsyncResult>();

	if (Query.IsEmpty())
	{
		Result->SetError(TEXT("Query must be non-empty."));
		return Result;
	}

	TSharedPtr<UE::SemanticSearch::IEmbeddingProvider> Provider =
		UE::SemanticSearch::ISemanticSearchModule::Get().GetEmbeddingProvider();
	if (!Provider.IsValid())
	{
		Result->SetError(TEXT("No embedding provider is registered with the SemanticSearch plugin."));
		return Result;
	}

	FCandidateSet Candidates;
	if (!PrepareCandidates(K, ClassFilter, PathRegexes, *Result, TEXT("Search"), Candidates))
	{
		return Result;
	}

	TSharedRef<FSearchRequest> Request = MakeShared<FSearchRequest>();
	Request->AsyncResult = TStrongObjectPtr<USemanticSearchAsyncResult>(Result);
	Request->Query = Query;
	Request->K = K;

	Provider->GenerateEmbeddingAsync(
		Query,
		[Request, IDFilter = MoveTemp(Candidates.IDFilter), IDToAsset = MoveTemp(Candidates.IDToAsset)]
		(UE::SemanticSearch::FEmbeddingResponse&& Response) mutable
		{
			auto Continuation = [Request, Response = MoveTemp(Response),
				IDFilter = MoveTemp(IDFilter), IDToAsset = MoveTemp(IDToAsset)]() mutable
			{
				if (!Response.ErrorMessage.IsEmpty())
				{
					Request->AsyncResult->SetError(FString::Printf(
						TEXT("Embedding generation failed: %s"), *Response.ErrorMessage));
					return;
				}
				RunSearchWithEmbedding(Request, MoveTemp(Response.Embedding),
					MoveTemp(IDFilter), MoveTemp(IDToAsset));
			};

			if (IsInGameThread())
			{
				Continuation();
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, [Continuation = MoveTemp(Continuation)]() mutable { Continuation(); });
			}
		},
		nullptr);

	return Result;
}

USemanticSearchAsyncResult* USemanticSearchToolset::FindSimilar(
	const FSoftObjectPath& AssetPath,
	const TArray<UClass*>& ClassFilter,
	const TArray<FString>& PathRegexes,
	int32 K)
{
	using namespace UE::SemanticSearchToolset;
	using namespace UE::SemanticSearchToolset::Private;

	USemanticSearchAsyncResult* Result = NewObject<USemanticSearchAsyncResult>();

	IAssetRegistry& AssetRegistry =
		FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	const FAssetData SourceAsset = AssetRegistry.GetAssetByObjectPath(AssetPath);
	if (!SourceAsset.IsValid())
	{
		Result->SetError(FString::Printf(
			TEXT("Asset '%s' could not be found. Expected an object path like '/Game/X/Foo.Foo'."),
			*AssetPath.ToString()));
		return Result;
	}

	if (UE::SemanticSearch::FHybridSearchIndex::Get().GetCachedIndexStats().VectorCount == 0)
	{
		Result->SetError(TEXT("The SemanticSearch index is empty — nothing to compare against."));
		return Result;
	}

	FCandidateSet Candidates;
	if (!PrepareCandidates(K, ClassFilter, PathRegexes, *Result, TEXT("FindSimilar"), Candidates))
	{
		return Result;
	}

	const int64 SourceID = UE::SemanticSearch::GetAssetIndexID(SourceAsset);

	// Exclude the source asset from results — callers don't want it as its own top match.
	Candidates.IDFilter.Remove(SourceID);
	Candidates.IDToAsset.Remove(SourceID);

	// If removing the source emptied the candidate set, short-circuit. An empty IDFilter
	// would otherwise be interpreted by SearchAsync as "search everything."
	if (Candidates.IDFilter.IsEmpty())
	{
		Result->SetValue(TArray<FSemanticSearchResult>());
		return Result;
	}

	TSharedRef<FSearchRequest> Request = MakeShared<FSearchRequest>();
	Request->AsyncResult = TStrongObjectPtr<USemanticSearchAsyncResult>(Result);
	// Empty query text: SearchAsync skips BM25 and returns vector-only results.
	Request->Query = FString();
	Request->K = K;

	UE::SemanticSearch::FHybridSearchIndex::Get().ExtractEmbeddingsAsync(
		{ SourceID },
		[Request, AssetPath, IDFilter = MoveTemp(Candidates.IDFilter), IDToAsset = MoveTemp(Candidates.IDToAsset)]
		(TArray<int64>&& IDs, TArray<float>&& Vectors, int32 Dimension) mutable
		{
			// ExtractEmbeddingsAsync guarantees game-thread delivery.
			if (Dimension <= 0 || IDs.IsEmpty())
			{
				Request->AsyncResult->SetError(FString::Printf(
					TEXT("Asset '%s' is not in the SemanticSearch index."), *AssetPath.ToString()));
				return;
			}

			// One ID requested → the first Dimension floats are our source embedding.
			TArray<float> Embedding(Vectors.GetData(), Dimension);
			RunSearchWithEmbedding(Request, MoveTemp(Embedding),
				MoveTemp(IDFilter), MoveTemp(IDToAsset));
		});

	return Result;
}
