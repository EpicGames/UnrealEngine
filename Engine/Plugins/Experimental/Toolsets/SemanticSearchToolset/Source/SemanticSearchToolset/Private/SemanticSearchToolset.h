// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "SemanticSearchToolset.generated.h"

class USemanticSearchAsyncResult;

// A single hit from a semantic search over the Content Browser's indexed assets.
USTRUCT()
struct FSemanticSearchResult
{
	GENERATED_BODY()

	// Full asset path, e.g. "/Game/Blueprints/BP_Foo.BP_Foo".
	UPROPERTY()
	FSoftObjectPath Path;

	// Class of the asset
	UPROPERTY()
	TObjectPtr<UClass> Class;

	// Caption produced at index time by the SemanticSearch plugin.
	UPROPERTY()
	FString Caption;
};

/**
 * Toolset exposing the SemanticSearch plugin's hybrid vector + BM25 asset search
 * to the AI Toolset Registry.
 */
UCLASS()
class USemanticSearchToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Run a semantic search over the Content Browser assets indexed by the SemanticSearch plugin.
	 *
	 * @param Query        Natural-language query. Must be non-empty.
	 * @param ClassFilter  If non-empty, array of classes to filter by (provide full ref path of the class).
	 *                     Passing a base class covers all its subclasses. The SemanticSearch index currently 
	 *  				   covers these base classes: Blueprint, StaticMesh, SkeletalMesh, Texture, 
	 * 					   Material, MaterialInstance. Pass an empty array to consider all indexed classes.
	 * @param PathRegexes  If non-empty, only assets whose soft object path matches at least one of
	 *                     these regular expressions are considered (e.g. "^/Game/Blueprints/.*").
	 *                     Pass an empty array to consider all indexed paths.
	 * @param K            Maximum number of results to return. Must be >= 1. Defaults to 10.
	 * @return             Array of FSemanticSearchResult sorted by relevance (highest Score first)
	 */
	UFUNCTION(meta = (AICallable), Category = "SemanticSearch")
	static USemanticSearchAsyncResult* Search(
		const FString& Query,
		const TArray<UClass*>& ClassFilter,
		const TArray<FString>& PathRegexes,
		int32 K = 10);

	/**
	 * Find assets whose embeddings are semantically similar to the given asset's embedding.
	 * Vector-only (no BM25). The source asset must already be indexed by the SemanticSearch
	 * plugin.
	 *
	 * @param AssetPath    SoftObjectPath of the reference asset.
	 * @param ClassFilter  Same semantics as Search::ClassFilter. See Search for the list of
	 *                     currently-indexed base classes. Pass an empty array for no class filter.
	 * @param PathRegexes  Same semantics as Search::PathRegexes. Pass an empty array for no path filter.
	 * @param K            Maximum number of results to return. Must be >= 1. Defaults to 10.
	 * @return             Array of FSemanticSearchResult sorted by relevance (highest Score first)
	 */
	UFUNCTION(meta = (AICallable), Category = "SemanticSearch")
	static USemanticSearchAsyncResult* FindSimilar(
		const FSoftObjectPath& AssetPath,
		const TArray<UClass*>& ClassFilter,
		const TArray<FString>& PathRegexes,
		int32 K = 10);
};
