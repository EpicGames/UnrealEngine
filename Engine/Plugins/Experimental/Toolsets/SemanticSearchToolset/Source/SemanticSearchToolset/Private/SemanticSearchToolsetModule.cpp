// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchToolset.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

class FSemanticSearchToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(USemanticSearchToolset::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		UToolsetRegistry::UnregisterToolsetClass(USemanticSearchToolset::StaticClass());
	}
};

IMPLEMENT_MODULE(FSemanticSearchToolsetModule, SemanticSearchToolset);
