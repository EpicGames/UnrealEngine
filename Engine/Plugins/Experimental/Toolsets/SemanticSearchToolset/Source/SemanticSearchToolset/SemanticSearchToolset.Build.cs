// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SemanticSearchToolset : ModuleRules
{
	public SemanticSearchToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;
		bUseUnity = true;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetRegistry",
			"CoreUObject",
			"Engine",
			"Json",
			"SemanticSearch",
			"ToolsetRegistry",
			"UnrealEd",
		});
	}
}
